// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NullDrv : ModuleRules
{
	public NullDrv(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.Add("Core");
        PrivateDependencyModuleNames.Add("RHI");
        PrivateDependencyModuleNames.Add("RenderCore");
	}
}
