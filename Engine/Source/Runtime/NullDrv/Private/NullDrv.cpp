// Copyright Epic Games, Inc. All Rights Reserved.

#include "NullDrv.h"
#include "Modules/ModuleManager.h"


bool FNullDynamicRHIModule::IsSupported()
{
	return true;
}


IMPLEMENT_MODULE(FNullDynamicRHIModule, NullDrv);
