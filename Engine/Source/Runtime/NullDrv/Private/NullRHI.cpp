// Copyright Epic Games, Inc. All Rights Reserved.

#include "NullRHI.h"
#include "Misc/CoreMisc.h"
#include "Containers/List.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RHICommandList.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GPUProfiler.h"

constexpr UE::RHI::GPUProfiler::FQueue NullRHIGPUProfilerQueue(UE::RHI::GPUProfiler::FQueue::EType::Graphics, 0, 0);

FNullDynamicRHI::FNullDynamicRHI()
{
	GMaxRHIShaderPlatform = ShaderFormatToLegacyShaderPlatform(FName(FPlatformMisc::GetNullRHIShaderFormat()));
	GMaxTextureDimensions = 16384;
	GMaxTextureMipCount = FPlatformMath::CeilLogTwo(GMaxTextureDimensions) + 1;
	GMaxTextureMipCount = FPlatformMath::Min<int32>( MAX_TEXTURE_MIP_COUNT, GMaxTextureMipCount );
}


void FNullDynamicRHI::Init()
{
	GMaxRHIFeatureLevel = GetMaxSupportedFeatureLevel(GMaxRHIShaderPlatform);

#if PLATFORM_WINDOWS
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_PCD3D_ES3_1;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = SP_PCD3D_SM5;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = SP_PCD3D_SM6;
#elif PLATFORM_MAC
    GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
    GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_METAL_ES3_1;
    GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_NumPlatforms;
    GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = SP_METAL_SM5;
    GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = SP_METAL_SM6;
#elif PLATFORM_LINUX // (see FVulkanGenericPlatform::SetupFeatureLevels)
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = SP_VULKAN_PCES3_1;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = SP_VULKAN_SM5;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM6] = SP_VULKAN_SM6;
#else
	GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel] = GMaxRHIShaderPlatform;
#endif
	
	GRHIVendorId = 1;

	check(!GIsRHIInitialized);

	// do not do this at least on dedicated server; clients with -NullRHI may need additional consideration
#if !WITH_EDITOR
	if (!IsRunningDedicatedServer())
#endif
	{
		FRenderResource::InitPreRHIResources();
	}

	GIsRHIInitialized = true;

	// Register a single dummy graphics GPU queue. This is required so that we can still submit the frame boundary events in RHIEndFrame.
	UE::RHI::GPUProfiler::InitializeQueues(MakeConstArrayView(&NullRHIGPUProfilerQueue, 1));
}


void FNullDynamicRHI::Shutdown()
{
}


/**
 * Return a shared large static buffer that can be used to return from any 
 * function that needs to return a valid pointer (but can be garbage data)
 */
void* FNullDynamicRHI::GetStaticBuffer(size_t Size)
{
#if !WITH_EDITOR
	static bool bLogOnce = false;

	if (!bLogOnce && (IsRunningDedicatedServer()))
	{
		UE_LOGF(LogRHI, Log, "NullRHI preferably does not allocate memory on the server. Try to change the caller to avoid doing allocs in when FApp::ShouldUseNullRHI() is true.");
		bLogOnce = true;
	}
#endif

	MemoryBuffer.Reserve(Size);
	return MemoryBuffer.GetData();
}

void* FNullDynamicRHI::GetStaticTextureBuffer(int32 SizeX, int32 SizeY, EPixelFormat Format, uint32& DestStride, uint64* OutLockedByteCount)
{
	size_t Size = CalculateImageBytes(SizeX, SizeY, 0, Format);
	if (OutLockedByteCount)
	{
		*OutLockedByteCount = Size;
	}
	DestStride = Size / SizeY;
	return GetStaticBuffer(Size);
}

void FNullDynamicRHI::RHIEndFrame(const FRHIEndFrameArgs& Args)
{
	// The RHI must publish the frame boundary events for engine stats to work correctly.
	UE::RHI::GPUProfiler::FEventStream EventStream(NullRHIGPUProfilerQueue);

	EventStream.Emplace<UE::RHI::GPUProfiler::FEvent::FFrameBoundary>(ERHIPipeline::Graphics, Args, FPlatformTime::Cycles64());

	UE::RHI::GPUProfiler::ProcessEvents(MakeArrayView(&EventStream, 1));
}

/** Value between 0-100 that determines the percentage of the vertical scan that is allowed to pass while still allowing us to swap when VSYNC'ed.
This is used to get the same behavior as the old *_OR_IMMEDIATE present modes. */
uint32 GPresentImmediateThreshold = 100;




// Suppress linker warning "warning LNK4221: no public symbols found; archive member will be inaccessible"
int32 NullRHILinkerHelper;
