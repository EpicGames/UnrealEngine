// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Dependencies.

#include "CoreMinimal.h"
#include "RHI.h"
#include "NullRHI.h"

/** Implements the NullDrv module as a dynamic RHI providing module. */
class FNullDynamicRHIModule
	: public IDynamicRHIModule
{
public:

	// IDynamicRHIModule

	virtual bool SupportsDynamicReloading() override { return false; }
	virtual bool IsSupported() override;

	virtual FDynamicRHI* CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::Num) override
	{
		return new FNullDynamicRHI();
	}
};
