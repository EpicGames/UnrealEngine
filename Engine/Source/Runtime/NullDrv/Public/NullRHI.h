// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ResourceArray.h"
#include "Serialization/LargeMemoryData.h"
#include "RHI.h"
#include "RHITypes.h"
#include "Async/TaskGraphInterfaces.h"
#include "RHITextureUtils.h"

struct Rect;

/** A null implementation of the dynamically bound RHI. */
class FNullDynamicRHI : public FDynamicRHIPSOFallback, public IRHICommandContextPSOFallback
{
public:

	FNullDynamicRHI();

	// FDynamicRHI interface.
	virtual void Init();
	virtual void Shutdown();
	virtual const TCHAR* GetName() override { return TEXT("Null"); }
	virtual ERHIInterfaceType GetInterfaceType() const override { return ERHIInterfaceType::Null; }

	virtual FSamplerStateRHIRef RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer) final override
	{ 
		return new FRHISamplerState(); 
	}
	virtual FRasterizerStateRHIRef RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer) final override
	{ 
		return new FRHIRasterizerState(); 
	}
	virtual FDepthStencilStateRHIRef RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer) final override
	{ 
		return new FRHIDepthStencilState(); 
	}
	virtual FBlendStateRHIRef RHICreateBlendState(const FBlendStateInitializerRHI& Initializer) final override
	{ 
		return new FRHIBlendState(); 
	}
	virtual FVertexDeclarationRHIRef RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements) final override
	{ 
		return new FRHIVertexDeclaration(); 
	}

	virtual FPixelShaderRHIRef RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc) final override
	{ 
		return new FRHIPixelShader(); 
	}

	virtual FVertexShaderRHIRef RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc) final override
	{ 
		return new FRHIVertexShader(); 
	}

	virtual FGeometryShaderRHIRef RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc) final override
	{ 
		return new FRHIGeometryShader(); 
	}

	virtual FComputeShaderRHIRef RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc) final override
	{ 
		return new FRHIComputeShader(); 
	}


	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, FRHIVertexShader* VertexShader, FRHIPixelShader* PixelShader, FRHIGeometryShader* GeometryShader) final override
	{ 
		return new FRHIBoundShaderState(); 
	}

#if PLATFORM_SUPPORTS_MESH_SHADERS && PLATFORM_USE_FALLBACK_PSO
	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(FRHIAmplificationShader* AmplificationShader, FRHIMeshShader* MeshShader, FRHIPixelShader* PixelShader) final override
	{
		return new FRHIBoundShaderState();
	}
#endif

	virtual void RHISetComputeShader(FRHIComputeShader* ComputeShader) final override
	{

	}

	virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override
	{

	}

	virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override
	{

	}

	virtual void RHICreateTransition(FRHITransition* Transition, const FRHITransitionCreateInfo& CreateInfo) final override
	{
	}

	virtual void RHIReleaseTransition(FRHITransition* Transition) final override
	{
	}

	virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions) final override
	{
	}

	virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions) final override
	{
	}

	virtual void RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) final override
	{

	}

	virtual FUniformBufferRHIRef RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation) final override
	{ 
		return new FRHIUniformBuffer(Layout); 
	}

	virtual void RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents) final override
	{

	}

	class FNullBuffer : public FRHIBuffer
	{
	public:
		FNullBuffer(const FRHIBufferCreateDesc& CreateDesc)
			: FRHIBuffer(CreateDesc)
		{
		}
	};

	struct FNullBufferInitializer : public FRHIBufferInitializer
	{
		FNullBufferInitializer(FRHICommandListBase& RHICmdList, FNullBuffer* Buffer, void* InWritableData, uint64 InWritableDataSize)
			: FRHIBufferInitializer(RHICmdList, Buffer, InWritableData, InWritableDataSize,
				[Buffer = TRefCountPtr<FNullBuffer>(Buffer)](FRHICommandListBase&) mutable
				{
					return TRefCountPtr<FRHIBuffer>(MoveTemp(Buffer));
				}
			)
		{
		}
	};

	[[nodiscard]]
	virtual FRHIBufferInitializer RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc) final override
	{
		if (CreateDesc.InitAction == ERHIBufferInitAction::ResourceArray && CreateDesc.InitialData)
		{
			CreateDesc.InitialData->Discard();
		}

		void* WritableData = nullptr;
		uint64 WritableDataSize = 0;

		if (CreateDesc.InitAction == ERHIBufferInitAction::Initializer)
		{
			WritableData = GetStaticBuffer(CreateDesc.Size);
			WritableDataSize = CreateDesc.Size;
		}

		FNullBuffer* Buffer = new FNullBuffer(CreateDesc);

		return FNullBufferInitializer(RHICmdList, Buffer, WritableData, WritableDataSize);
	}

	virtual void* LockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 SizeRHI, EResourceLockMode LockMode) final override
	{
		return GetStaticBuffer(Buffer->GetSize());
	}

	virtual void UnlockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override
	{

	}

	virtual void RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos) final override
	{

	}

	virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) final override
	{

	}

	virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) final override
	{

	}

	virtual FRHICalcTextureSizeResult RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex) override final
	{
		return {};
	}

	virtual void RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats) final override
	{

	}

	virtual bool RHIGetTextureMemoryVisualizeData(FColor* TextureData,int32 SizeX,int32 SizeY,int32 Pitch,int32 PixelSize) final override
	{ 
		return false; 
	}

	class FNullTexture : public FRHITexture
	{
	public:
		FNullTexture(const FRHITextureCreateDesc& InDesc)
			: FRHITexture(InDesc)
		{}
	};

	struct FNullTextureInitializer : public FRHITextureInitializer
	{
		FNullTextureInitializer(FRHICommandListBase& RHICmdList, FRHITexture* Texture, void* InWritableData, uint64 InWritableDataSize)
			: FRHITextureInitializer(RHICmdList, Texture, InWritableData, InWritableDataSize,
				[Texture = TRefCountPtr<FRHITexture>(Texture)](FRHICommandListBase&) mutable
				{
					return MoveTemp(Texture);
				},
				[Texture = TRefCountPtr<FRHITexture>(Texture), WritableData = InWritableData](FRHITextureInitializer::FSubresourceIndex SubresourceIndex)
				{
					const FRHITextureDesc& TextureDesc = Texture->GetDesc();

					uint64 SubresourceStride = 0;
					uint64 SubresourceSize = 0;
					const uint64 Offset = UE::RHITextureUtils::CalculateSubresourceOffset(TextureDesc, SubresourceIndex.FaceIndex, SubresourceIndex.ArrayIndex, SubresourceIndex.MipIndex, SubresourceStride, SubresourceSize);

					FRHITextureSubresourceInitializer Result{};
					Result.Data = reinterpret_cast<uint8*>(WritableData) + Offset;
					Result.Stride = SubresourceStride;
					Result.Size = SubresourceSize;

					return Result;
				}
			)
		{
		}
	};

	[[nodiscard]]
	virtual FRHITextureInitializer RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc)
	{
		if (CreateDesc.InitAction == ERHITextureInitAction::BulkData && CreateDesc.BulkData)
		{
			CreateDesc.BulkData->Discard();
		}

		void* WritableData = nullptr;
		uint64 WritableDataSize = 0;

		if (CreateDesc.InitAction == ERHITextureInitAction::Initializer)
		{
			WritableDataSize = UE::RHITextureUtils::CalculateTextureSize(CreateDesc);
			WritableData = GetStaticBuffer(WritableDataSize);
		}

		FRHITexture* Texture = new FNullTexture(CreateDesc);

		return FNullTextureInitializer(RHICmdList, Texture, WritableData, WritableDataSize);
	}

	virtual FTextureRHIRef RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent) final override
	{ 
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(DebugName, SizeX, SizeY, (EPixelFormat)Format)
			.SetClearValue(FClearValueBinding::None)
			.SetFlags(Flags)
			.SetNumMips(NumMips)
			.SetInitialState(InResourceState);
		OutCompletionEvent = nullptr;
		return new FNullTexture(Desc);
	}

	virtual FShaderResourceViewRHIRef RHICreateShaderResourceView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
	{
		return new FRHIShaderResourceView(Resource, ViewDesc);
	}

	virtual FUnorderedAccessViewRHIRef RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
	{
		return new FRHIUnorderedAccessView(Resource, ViewDesc);
	}

	virtual uint32 RHIComputeMemorySize(FRHITexture* TextureRHI) final override
	{ 
		return 0; 
	}

	virtual FRHILockTextureResult RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override
	{
		FRHITexture* Texture = Arguments.Texture;

		FRHILockTextureResult Result{};
		Result.Data = GetStaticTextureBuffer(Texture->GetSizeX(), Texture->GetSizeY(), Texture->GetFormat(), Result.Stride, &Result.ByteCount);

		return Result;
	}
	virtual void RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override
	{
	}

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	virtual void RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override
	{
	}
#endif

	virtual void RHIUpdateTexture2D(FRHICommandListBase&, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData) final override
	{

	}
	virtual void RHIUpdateTexture3D(FRHICommandListBase&, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData) final override
	{

	}

	virtual void RHICopyTexture(FRHITexture* SourceTexture, FRHITexture* DestTexture, const FRHICopyTextureInfo& CopyInfo) final override
	{

	}

	virtual void RHICopyBufferRegion(FRHIBuffer* DestBuffer, uint64 DstOffset, FRHIBuffer* SourceBuffer, uint64 SrcOffset, uint64 NumBytes) final override
	{
	}

	virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const TCHAR* Name) final override
	{

	}

	virtual void RHIReadSurfaceData(FRHITexture* Texture,FIntRect Rect,TArray<FColor>& OutData,FReadSurfaceDataFlags InFlags) final override
	{ 
		OutData.AddZeroed(Rect.Width() * Rect.Height()); 
	}


	virtual FGPUFenceRHIRef RHICreateGPUFence(const FName& Name)
	{
		class FNullGPUFence final : public FRHIGPUFence
		{
		public:
			FNullGPUFence(const FName& Name)
				: FRHIGPUFence(Name)
			{}

			virtual void Clear() override
			{
			}

			virtual bool Poll() const override
			{
				return true;
			}

			virtual void Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const override
			{
			}
		};
		return new FNullGPUFence(Name);
	}

	virtual void RHIMapStagingSurface(FRHITexture* Texture, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex) final override
	{

	}


	virtual void RHIUnmapStagingSurface(FRHITexture* Texture, uint32 GPUIndex) final override
	{

	}

	virtual void RHIReadSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,TArray<FFloat16Color>& OutData,ECubeFace CubeFace,int32 ArrayIndex,int32 MipIndex) final override
	{

	}

	virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,FIntPoint ZMinMax,TArray<FFloat16Color>& OutData) final override
	{

	}



	virtual FRenderQueryRHIRef RHICreateRenderQuery(ERenderQueryType QueryType) final override
	{ 
		return new FRHIRenderQuery(); 
	}

	virtual void RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery) final override
	{

	}
	virtual void RHIEndRenderQuery(FRHIRenderQuery* RenderQuery) final override
	{

	}


	virtual bool RHIGetRenderQueryResult(FRHIRenderQuery* RenderQuery, uint64& OutResult, bool bWait, uint32 GPUIndex = INDEX_NONE) final override
	{ 
		return true; 
	}

	virtual void RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* Viewport, const FRHIPresentArgs& PresentArgs) final override
	{
	}

	virtual FTextureRHIRef RHIGetViewportBackBuffer(FRHIViewport* Viewport) final override
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FNullDynamicRHI::RHIGetViewportBackBuffer"), 1, 1, PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::RenderTargetable);

		return new FNullTexture(Desc);
	}

	virtual void RHIEndFrame(const FRHIEndFrameArgs& Args) final override;

	virtual void RHIAliasTextureResources(FTextureRHIRef& DestTexture, FTextureRHIRef& SrcTexture) final override
	{

	}

	virtual FViewportRHIRef RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override
	{ 
		return new FRHIViewport(); 
	}

	virtual void RHIResizeViewport(FRHIViewport* Viewport, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override
	{

	}

	virtual void RHITick(float DeltaTime) final override
	{

	}

	virtual void RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset) final override
	{
	}

	virtual void RHISetRasterizerState(FRHIRasterizerState* NewState) final override
	{

	}

	virtual void RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ) final override
	{

	}

	virtual void RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY) final override
	{
	}

	virtual void RHISetBoundShaderState(FRHIBoundShaderState* BoundShaderState) final override
	{
	}

	virtual void RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override
	{
	}

	virtual void RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override
	{
	}

	virtual void RHISetDepthStencilState(FRHIDepthStencilState* NewState, uint32 StencilRef) final override
	{

	}

	virtual void RHISetBlendState(FRHIBlendState* NewState, const FLinearColor& BlendFactor) final override
	{

	}

	virtual void RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName)
	{
	}

	virtual void RHIEndRenderPass()
	{
	}

	virtual void RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances) final override
	{
	}

	virtual void RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override
	{
	}

	virtual void RHIDrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances) final override
	{
	}

	virtual void RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override
	{
	}

	virtual void RHIMultiDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset, FRHIBuffer* CountBuffer, uint32 CountBufferOffset, uint32 MaxDrawArguments) final override
	{
	}

	virtual void RHIBlockUntilGPUIdle() final override
	{
	}
	virtual bool RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate) final override
	{ 
		return false; 
	}
	virtual void RHIGetSupportedResolution(uint32& Width, uint32& Height) final override
	{

	}
	virtual void RHIEnableDepthBoundsTest(bool bEnable) final override
	{
	}
	virtual void RHISetDepthBounds(float MinDepth, float MaxDepth) final override
	{
	}
	virtual void RHISetShadingRate(EVRSShadingRate ShadingRate, EVRSRateCombiner Combiner) final override
	{
	}
	virtual void* RHIGetNativeDevice() final override
	{ 
		return 0; 
	}

	virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) final override
	{
	}

	virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* UniformBuffer) final override
	{
	}

#if WITH_RHI_BREADCRUMBS
	virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override
	{
	}
	virtual void RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override
	{
	}
#endif

	virtual class IRHICommandContext* RHIGetDefaultContext() final override
	{ 
		return this; 
	}

	virtual IRHIComputeContext* RHIGetCommandContext(ERHIPipeline Pipeline) final override
	{
		return nullptr;
	}

	virtual FRHIFinalizeContextsResult RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args) final override
	{
		return {};
	}

	virtual void RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args) final override
	{
	}

private:
	FLargeMemoryData MemoryBuffer;

	/** Allocates a static buffer for RHI functions to return as a write destination. */
	void* GetStaticBuffer(size_t Size);
	void* GetStaticTextureBuffer(int32 SizeX, int32 SizeY, EPixelFormat Format, uint32& DestStride, uint64* OutLockedByteCount = nullptr);
};
